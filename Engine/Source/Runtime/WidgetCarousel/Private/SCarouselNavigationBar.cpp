// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCarouselNavigationBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Application/SlateWindowHelper.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "WidgetCarousel"

int32 SCarouselNavigationBar::GetItemCount()
{
	return ItemCount;
}

void SCarouselNavigationBar::SetItemCount(int32 InItemCount)
{
	ItemCount = InItemCount;
	BuildScrollBar();
}

void SCarouselNavigationBar::Construct(const FArguments& InArgs)
{
	Style = InArgs._Style;
	OnSelectedIndexChanged = InArgs._OnSelectedIndexChanged;
	ItemCount = InArgs._ItemCount;
	CurrentItemIndex = InArgs._CurrentItemIndex;
	CurrentSlideAmount = InArgs._CurrentSlideAmount;

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SFxWidget)
			.IgnoreClipping(false)
			.VisualOffset(this, &SCarouselNavigationBar::GetPlaceHolderPosition)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SImage)
					.Image(&Style->HighlightBrush)
				]
			]
		]
		+ SOverlay::Slot()
		[
			SAssignNew(WidgetScrollBox, SHorizontalBox)
		]
	];

	BuildScrollBar();
}

void SCarouselNavigationBar::BuildScrollBar()
{
	// May change this to have the scroll bar get their images via delegate rather than rebuild
	WidgetScrollBox->ClearChildren();

	if (ItemCount > 1)
	{
		for (int32 Index = 0; Index < ItemCount; Index++)
		{
			const FButtonStyle* ButtonStyle;
			if (Index == 0)
			{
				ButtonStyle = &Style->LeftButtonStyle;
			}
			else if (Index == ItemCount - 1)
			{
				ButtonStyle = &Style->RightButtonStyle;
			}
			else
			{
				ButtonStyle = &Style->CenterButtonStyle;
			}

			WidgetScrollBox->AddSlot()
			.Padding(0)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(ButtonStyle)
				.OnClicked(this, &SCarouselNavigationBar::HandleItemButtonClicked, Index)
				.Cursor(EMouseCursor::Hand)
			];
		}
	}
}

FReply SCarouselNavigationBar::HandleItemButtonClicked(int32 ItemIndex)
{
	OnSelectedIndexChanged.ExecuteIfBound(ItemIndex);
	BuildScrollBar();
	return FReply::Handled();
}

FVector2D SCarouselNavigationBar::GetPlaceHolderPosition() const
{
	float WidgetSize = 1.f / (float)ItemCount;

	float NewPos = CurrentItemIndex.Get() * WidgetSize;
	NewPos -= CurrentSlideAmount.Get() * WidgetSize;

	return FVector2D(NewPos, 0.f);
}

#undef LOCTEXT_NAMESPACE
