// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetCarouselModule.h"
#include "Modules/ModuleManager.h"

void FWidgetCarouselModule::StartupModule()
{

}

void FWidgetCarouselModule::ShutdownModule()
{

}

IMPLEMENT_MODULE(FWidgetCarouselModule, WidgetCarousel);
