// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetCarouselStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetCarouselStyle)

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

/**
	Navigation button style
*/
FWidgetCarouselNavigationButtonStyle::FWidgetCarouselNavigationButtonStyle()
{
}

const FName FWidgetCarouselNavigationButtonStyle::TypeName(TEXT("FWidgetCarouselNavigationButtonStyle"));

const FWidgetCarouselNavigationButtonStyle& FWidgetCarouselNavigationButtonStyle::GetDefault()
{
	static FWidgetCarouselNavigationButtonStyle Default;
	return Default;
}

void FWidgetCarouselNavigationButtonStyle::GetResources(TArray< const FSlateBrush* > & OutBrushes) const
{
	OutBrushes.Add(&NavigationButtonLeftImage);
	OutBrushes.Add(&NavigationButtonRightImage);
	InnerButtonStyle.GetResources(OutBrushes);
}

/**
Navigation bar style
*/
FWidgetCarouselNavigationBarStyle::FWidgetCarouselNavigationBarStyle()
{
}

const FName FWidgetCarouselNavigationBarStyle::TypeName(TEXT("FWidgetCarouselNavigationBarStyle"));

const FWidgetCarouselNavigationBarStyle& FWidgetCarouselNavigationBarStyle::GetDefault()
{
	static FWidgetCarouselNavigationBarStyle Default;
	return Default;
}

void FWidgetCarouselNavigationBarStyle::GetResources(TArray< const FSlateBrush* > & OutBrushes) const
{
	OutBrushes.Add(&HighlightBrush);
	LeftButtonStyle.GetResources(OutBrushes);
	CenterButtonStyle.GetResources(OutBrushes);
	RightButtonStyle.GetResources(OutBrushes);
}

/**
	Module style set
*/
TSharedPtr< FSlateStyleSet > FWidgetCarouselModuleStyle::WidgetCarouselStyleInstance = NULL;

void FWidgetCarouselModuleStyle::Initialize()
{
	if ( !WidgetCarouselStyleInstance.IsValid() )
	{
		WidgetCarouselStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle( *WidgetCarouselStyleInstance );
	}
}

void FWidgetCarouselModuleStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle( *WidgetCarouselStyleInstance );
	ensure( WidgetCarouselStyleInstance.IsUnique() );
	WidgetCarouselStyleInstance.Reset();
}

FName FWidgetCarouselModuleStyle::GetStyleSetName()
{
	static FName StyleSetName("WidgetCarousel");
	return StyleSetName;
}

TSharedRef< FSlateStyleSet > FWidgetCarouselModuleStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("WidgetCarouselStyle"));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/WidgetCarousel"));

	const FButtonStyle DefaultButton = FButtonStyle()
		.SetNormalPadding(0)
		.SetPressedPadding(0);

	FLinearColor PrimaryCallToActionColor = FLinearColor(1.0f, 0.7372f, 0.05637f);
	FLinearColor PrimaryCallToActionColorHovered = FLinearColor(1.0f, 0.83553f, 0.28445f);
	FLinearColor PrimaryCallToActionColorPressed = FLinearColor(1.0f, 0.66612f, 0.0012f);

	Style->Set("CarouselNavigationButton", FWidgetCarouselNavigationButtonStyle()
		.SetInnerButtonStyle(FButtonStyle(DefaultButton)
		.SetNormal(BOX_BRUSH("WhiteBox_7px_CornerRadius", FVector2D(17, 17), FMargin(0.5), PrimaryCallToActionColor))
			.SetPressed(BOX_BRUSH("WhiteBox_7px_CornerRadius", FVector2D(17, 17), FMargin(0.5), PrimaryCallToActionColorPressed))
			.SetHovered(BOX_BRUSH("WhiteBox_7px_CornerRadius", FVector2D(17, 17), FMargin(0.5), PrimaryCallToActionColorHovered)))
		.SetNavigationButtonLeftImage(IMAGE_BRUSH("Arrow-Left", FVector2D(25.0f, 42.0f)))
		.SetNavigationButtonRightImage(IMAGE_BRUSH("Arrow-Right", FVector2D(25.0f, 42.0f))));

	Style->Set("CarouselNavigationBar", FWidgetCarouselNavigationBarStyle()
		.SetHighlightBrush(IMAGE_BRUSH("CarouselNavMarker", FVector2D(80.0f, 20.0f), PrimaryCallToActionColor))
		.SetLeftButtonStyle(FButtonStyle(DefaultButton)
		.SetNormal(IMAGE_BRUSH("CarouselNavLeft", FVector2D(80.0f, 20.0f), FLinearColor::White))
		.SetHovered(IMAGE_BRUSH("CarouselNavLeft", FVector2D(80.0f, 20.0f), PrimaryCallToActionColor))
			.SetPressed(IMAGE_BRUSH("CarouselNavLeft", FVector2D(80.0f, 20.0f), PrimaryCallToActionColorPressed)))
		.SetCenterButtonStyle(FButtonStyle()
		.SetNormal(IMAGE_BRUSH("CarouselNavCenter", FVector2D(80.0f, 20.0f), FLinearColor::White))
		.SetHovered(IMAGE_BRUSH("CarouselNavCenter", FVector2D(80.0f, 20.0f), PrimaryCallToActionColor))
			.SetPressed(IMAGE_BRUSH("CarouselNavCenter", FVector2D(80.0f, 20.0f), PrimaryCallToActionColorPressed)))
		.SetRightButtonStyle(FButtonStyle()
		.SetNormal(IMAGE_BRUSH("CarouselNavRight", FVector2D(80.0f, 20.0f), FLinearColor::White))
		.SetHovered(IMAGE_BRUSH("CarouselNavRight", FVector2D(80.0f, 20.0f), PrimaryCallToActionColor))
			.SetPressed(IMAGE_BRUSH("CarouselNavRight", FVector2D(80.0f, 20.0f), PrimaryCallToActionColorPressed))));

	Style->Set("WidgetBackground", new FSlateColorBrush(FLinearColor(0, 0, 0, 0.6f)));

	return Style;
}

void FWidgetCarouselModuleStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FWidgetCarouselModuleStyle::Get()
{
	return *WidgetCarouselStyleInstance;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH

