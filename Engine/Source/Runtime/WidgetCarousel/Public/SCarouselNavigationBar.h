// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "WidgetCarouselStyle.h"

#define LOCTEXT_NAMESPACE "WidgetCarousel"

class SHorizontalBox;

/**
* A horizontal bar of buttons for navigating to a specific item in the widget carousel.
*/
class SCarouselNavigationBar : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnSelectedIndexChanged, int32)

public:
	SLATE_BEGIN_ARGS(SCarouselNavigationBar)
		: _Style() 
		, _OnSelectedIndexChanged()
		, _ItemCount(0)
		, _CurrentItemIndex(0)
		, _CurrentSlideAmount(0.0f)
		
	{}

		SLATE_STYLE_ARGUMENT(FWidgetCarouselNavigationBarStyle, Style)

		/** Called when the selected index changes. */
		SLATE_EVENT(FOnSelectedIndexChanged, OnSelectedIndexChanged)

		/** The number of items to generate navigation buttons for. */
		SLATE_ARGUMENT(int32, ItemCount)

		/** The index of the currently selected item. */
		SLATE_ATTRIBUTE(int32, CurrentItemIndex)

		/** The amount of offset which should be applied to the selected item indicator. */
		SLATE_ATTRIBUTE(float, CurrentSlideAmount)

	SLATE_END_ARGS()

	WIDGETCAROUSEL_API int32 GetItemCount();

	WIDGETCAROUSEL_API void SetItemCount(int32 InItemCount);

	WIDGETCAROUSEL_API void Construct(const FArguments& InArgs);

private:
	void BuildScrollBar();

	FReply HandleItemButtonClicked(int32 ItemIndex);

	FVector2D GetPlaceHolderPosition() const;

private:
	const FWidgetCarouselNavigationBarStyle* Style;
	TSharedPtr<SHorizontalBox> WidgetScrollBox;
	FOnSelectedIndexChanged OnSelectedIndexChanged;
	int32 ItemCount;
	TAttribute<int32> CurrentItemIndex;
	TAttribute<float> CurrentSlideAmount;
};

#undef LOCTEXT_NAMESPACE
