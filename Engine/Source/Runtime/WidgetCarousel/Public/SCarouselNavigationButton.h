// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/StyleDefaults.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "WidgetCarouselStyle.h"
#include "HAL/PlatformProcess.h"
#include "Widgets/Layout/SSpacer.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "WidgetCarousel"

/**
 * A widget used to navigate the carousel widget
 */
class SCarouselNavigationButton
	: public SCompoundWidget
{
	DECLARE_DELEGATE(FOnBeginPeek)
	DECLARE_DELEGATE(FOnEndPeek)

public:

	class SPeekBorder 
		: public SBorder
	{
	public:

		SLATE_BEGIN_ARGS(SPeekBorder)
			: _OnBeginPeek()
			, _OnEndPeek()
			, _HAlign(HAlign_Fill)
			, _VAlign(VAlign_Fill)
			, _Padding(FMargin(0))
		{}
			/** Called when the button is hovered */
			SLATE_EVENT(FOnBeginPeek, OnBeginPeek)
			SLATE_EVENT(FOnEndPeek, OnEndPeek)

			SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
			SLATE_ARGUMENT(EVerticalAlignment, VAlign)
			SLATE_ATTRIBUTE(FMargin, Padding)

			SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			OnBeginPeek = InArgs._OnBeginPeek;
			OnEndPeek = InArgs._OnEndPeek;

			SBorder::Construct(SBorder::FArguments()
				.HAlign(InArgs._HAlign)
				.VAlign(InArgs._VAlign)
				.Padding(InArgs._Padding)
				.BorderImage(FStyleDefaults::GetNoBrush())
				[
					InArgs._Content.Widget
				]);
		}

		void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
		{
			if (FPlatformApplicationMisc::IsThisApplicationForeground())
			{
				if (OnBeginPeek.IsBound() == true)
				{
					OnBeginPeek.Execute();
				}
			}
			SBorder::OnMouseEnter(MyGeometry, MouseEvent);
		}

		void OnMouseLeave(const FPointerEvent& MouseEvent)
		{
			if (FPlatformApplicationMisc::IsThisApplicationForeground())
			{
				if (OnEndPeek.IsBound() == true)
				{
					OnEndPeek.Execute();
				}
			}
			SBorder::OnMouseLeave(MouseEvent);
		}

	private:

		FOnBeginPeek OnBeginPeek;
		FOnEndPeek OnEndPeek;
	};

public:

	enum class ENavigationButtonDirection
	{
		Left,
		Right
	};

	SLATE_BEGIN_ARGS(SCarouselNavigationButton)
		: _Style()
		{}

		SLATE_STYLE_ARGUMENT(FWidgetCarouselNavigationButtonStyle, Style)

		/** Called when the button is clicked */
		SLATE_EVENT(FOnClicked, OnClicked)

		/** Specifies which direction should be used to setup the button's visuals and layout. */
		SLATE_ARGUMENT(ENavigationButtonDirection, Direction)

		/** Called when the carousel should shift it's image to give a preview of the next image */
		SLATE_EVENT(FOnBeginPeek, OnBeginPeek)
		SLATE_EVENT(FOnEndPeek, OnEndPeek)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ImageTransparency = 0.f;
		Style = InArgs._Style;

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(150.0f)
			[
				SNew(SPeekBorder)
				.OnBeginPeek(InArgs._OnBeginPeek)
				.OnEndPeek(InArgs._OnEndPeek)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
						.Visibility(InArgs._Direction == ENavigationButtonDirection::Left ? EVisibility::Collapsed : EVisibility::Visible)
					]
					+SHorizontalBox::Slot()
					.HAlign(InArgs._Direction == ENavigationButtonDirection::Left ? HAlign_Left : HAlign_Right)
					.AutoWidth()
					[
						SAssignNew(HiddenButton, SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTipText(InArgs._ToolTipText)
						.OnClicked(InArgs._OnClicked)
						.ButtonStyle(FCoreStyle::Get(), "NoBorder")
						.Cursor(EMouseCursor::Hand)
						[
							SAssignNew(VisibleButton, SButton)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.ToolTipText(InArgs._ToolTipText)
							.OnClicked(InArgs._OnClicked)
							.ButtonStyle(&Style->InnerButtonStyle)
							.ContentPadding(FMargin(15, 30))
							.ButtonColorAndOpacity(this, &SCarouselNavigationButton::GetButtonColor)
							.Cursor(EMouseCursor::Hand)
							[
								SNew(SBox)
								.HeightOverride(42.0f)
								.WidthOverride(25.0f)
								[
									SNew(SImage)
									.Image(InArgs._Direction == ENavigationButtonDirection::Left ?
										&Style->NavigationButtonLeftImage : &Style->NavigationButtonRightImage)
									.ColorAndOpacity(this, &SCarouselNavigationButton::GetButtonImageColor)
								]
							]
						]
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SSpacer)
						.Visibility(InArgs._Direction == ENavigationButtonDirection::Left ? EVisibility::Visible : EVisibility::Collapsed)
					]
				]
			]
		];
	}

private:

	FSlateColor GetButtonColor() const
	{
		if (HiddenButton->IsHovered())
		{
			if (!VisibleButton->IsHovered())
			{
				FLinearColor Color = FLinearColor::White;
				Color.A = ImageTransparency;
				return Color;
			}
			else
			{
				return FLinearColor::White;
			}
		}

		return FLinearColor(0.0f, 0.0f, 0.0f, ImageTransparency * 0.5f);
	}

	FSlateColor GetButtonImageColor() const
	{
		if (HiddenButton->IsHovered() || VisibleButton->IsHovered())
		{
			if (!VisibleButton->IsHovered())
			{
				return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
			}

			return FLinearColor(0.0f, 0.0f, 0.0f, ImageTransparency);
		}

		return FLinearColor(1.0f, 1.0f, 1.0f, ImageTransparency);
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		//SButton::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		static const float BlendSpeed = 2.0f;

		float DesiredBlendSpeed = BlendSpeed * InDeltaTime;

		if (IsHovered())
		{
			if (ImageTransparency != 1.f && FPlatformApplicationMisc::IsThisApplicationForeground())
			{
				ImageTransparency = FMath::Min<float>(ImageTransparency + DesiredBlendSpeed, 1.f);
			}
			else
			{
				ImageTransparency = 1.f;
			}
		}
		else
		{
			if (ImageTransparency != 0.f && FPlatformApplicationMisc::IsThisApplicationForeground())
			{
				ImageTransparency = FMath::Max<float>(ImageTransparency - DesiredBlendSpeed, 0.f);
			}
			else
			{
				ImageTransparency = 0.f;
			}
		}
	}

private:
	const FWidgetCarouselNavigationButtonStyle* Style;
	TSharedPtr<SButton> HiddenButton;
	TSharedPtr<SButton> VisibleButton;

	float ImageTransparency;
	ENavigationButtonDirection Direction;
};

#undef LOCTEXT_NAMESPACE
