// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SWidgetCarousel.h: Declares the SWidgetCarousel class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateEnums.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Animation/CurveSequence.h"
#include "Widgets/Layout/SFxWidget.h"

/** The desired Carousel scroll direction */
namespace EWidgetCarouselScrollDirection
{
	enum Type
	{
		// Scroll the carousel left.
		Carousel_Left,
		// Scroll the carousel right.
		Carousel_Right,
		// Center the carousel
		Carousel_Center,
	};
}

/** The Carousel display widget that gets animated by the carousel widget */
struct FCarouselDisplayItem : public TSharedFromThis<FCarouselDisplayItem>
{
	/**
	 * Construct the display widget.
	 */
	FCarouselDisplayItem()
		: OpacityValue( 0.f )
		, SlideValue( 0.f )
		, DesiredOpacityValue( 0.f )
		, SlideValueLeftLimit( -1.0f )
		, SlideValueRightLimit( 1.0f )
		, PeakValueLeftLimit(-0.05f)
		, PeakValueRightLimit(0.05f)
		, OptimalSlideValue( 0.0f )
		, DesiredSlideValue( 0.f )
		, bTransition( false )
		, bPeak(false)
		, bFade( false )
		, MoveSpeed( 5.0f )
		, PeakSpeed(.2f)
		, PeakDistance(0.05f)
		, FadeRate( 2.0f )
		, Visibility(EVisibility::Visible)
	{
		// Create default widget
		FXWidget = SNew(SFxWidget)
			.IgnoreClipping(false);

		SlideInCurve = FCurveSequence();
		SlideInCurve.AddCurve(0.1f, 0.5f);
	}

	/**
	 * Scroll in the widget.
	 * @ScrollDirection The scroll direction.
	 */
	void ScrollIn( EWidgetCarouselScrollDirection::Type ScrollDirection )
	{
		bTransition = true;
		bPeak = false;
		bFade = FadeRate != 0;
		DesiredSlideValue = OptimalSlideValue;
		DesiredOpacityValue = 1.0f;
	}

	/**
	 * Scroll out the widget.
	 * @param ScrollDirection - The scroll direction.
	 */
	void PeakIn(EWidgetCarouselScrollDirection::Type ScrollDirection)
	{
		if (!bTransition)
		{
			SlideInCurve.JumpToEnd();
			SlideInCurve.PlayReverse(FXWidget.ToSharedRef());
			bPeak = true;
			bFade = FadeRate != 0;
			switch (ScrollDirection)
			{
				case EWidgetCarouselScrollDirection::Carousel_Left:
				{
					DesiredSlideValue = PeakValueLeftLimit;
				}
				break;
				case EWidgetCarouselScrollDirection::Carousel_Right:
				{
					DesiredSlideValue = PeakValueRightLimit;
				}
				break;
				case EWidgetCarouselScrollDirection::Carousel_Center:
				{
					DesiredSlideValue = OptimalSlideValue;
				}
				break;
			}
			DesiredOpacityValue = 0.0f;
		}
	}

	/**
	 * Tick the widget - animate the slide / fade.
	 * @param DeltaTime - The delta time.
	 */
	void Tick( float DeltaTime )
	{
		if ( bTransition )
		{
			bTransition = BlendWidget(DeltaTime, DesiredSlideValue, MoveSpeed, SlideValue);
			SetSlide( SlideValue );
		}
		else if ( bPeak )
		{
			bPeak = BlendWidget(DeltaTime, DesiredSlideValue, PeakSpeed, SlideValue);
			SetSlide(SlideValue);
		}

		if ( bFade )
		{
			bFade = BlendWidget(DeltaTime, DesiredOpacityValue, FadeRate, OpacityValue);
			SetOpacity(OpacityValue);
		}
	}

	/**
	 * Blend the widget fade / transform to the desired value.
	 * @param DeltaTime - The delta time.
	 * @param DesiredValue - The desired value.
	 * @param Speed - Speed to transform.
	 * @param OutCurrentValue - The value to set.
	 * @return True if still blending, false otherwise
	 */
	bool BlendWidget( const float& DeltaTime, const float& DesiredValue, const float& Speed, float& OutCurrentValue )
	{
		bool bStillBlending = true;

		float DesiredBlendSpeed = Speed * DeltaTime;
		float BlendDif = OutCurrentValue - DesiredValue;

		if (bPeak)
		{
			if (FMath::Abs(BlendDif) > PeakDistance + (float)FLOAT_NORMAL_THRESH)
			{
				SlideInCurve.JumpToEnd();
				SlideInCurve.PlayReverse(FXWidget.ToSharedRef());
			}
			float Lerp = FMath::Max(SlideInCurve.GetLerp(), 0.1f);
			DesiredBlendSpeed = Speed * Lerp * DeltaTime;
		}

		if (FMath::Abs(DesiredBlendSpeed) > FMath::Abs(BlendDif))
		{
			OutCurrentValue = DesiredValue;
			bStillBlending = false;
		}
		else
		{
			// Reverse Direction if sliding left
			if ( BlendDif > 0.f )
			{
				DesiredBlendSpeed *= -1.f;
			}

			OutCurrentValue += DesiredBlendSpeed;
		}

		return bStillBlending;
	}

	/**
	 * Set the content of the widget to display.
	 * @param InContent - The widget to display.
	 */
	void SetWidgetContent( TSharedRef<SWidget> InContent )
	{
		FXWidget = SNew( SFxWidget )
		.IgnoreClipping(false)
		.VisualOffset( this, &FCarouselDisplayItem::GetSlide )
		[
//			SNew(SOverlay)
//			+ SOverlay::Slot()
//			[
				InContent
//			]
//			+ SOverlay::Slot()
//			.HAlign(HAlign_Right)
//			[
//				SNew(STextBlock)
//				.Text(this, &FCarouselDisplayItem::GetPositionText)
//			]
//			+ SOverlay::Slot()
//			.HAlign(HAlign_Left)
//			[
//				SNew(STextBlock)
//				.Text(this, &FCarouselDisplayItem::GetPositionText)
//			]
		];
	}

	TSharedRef<SWidget> GetWidgetContent() const
	{
		if (FXWidget.IsValid())
		{
			FChildren* Children = FXWidget->GetChildren();
			if (Children && Children->Num() > 0)
			{
				return Children->GetChildAt(0);
			}		
		}

		return SNullWidget::NullWidget;
	}

//	FText GetPositionText() const
//	{
//		return FText::FromString(FString::Printf(TEXT("%.2f, %.2f, %.2f"), SlideValue, DesiredSlideValue, SlideInCurve.GetLerp()));
//	}

	/**
	 * Get the FX widget to display.
	 * @return The FX widget.
	 */
	TSharedPtr< SFxWidget > GetSFXWidget()
	{
		return FXWidget;
	}

	/**
	 * Get the widget opacity.
	 * @param InOpacity - the desired opacity.
	 */
	void SetOpacity( float InOpacity )
	{
		OpacityValue = InOpacity;
		FXWidget->SetColorAndOpacity( FLinearColor(1.f,1.f,1.f, OpacityValue ) );
	}

	void SetVisibility(EVisibility InVisibility)
	{
		Visibility = InVisibility;
		FXWidget->SetVisibility(InVisibility);
	}

	/**
	 * Set the slide position.
	 * @param InSlide - the desired slide position.
	 * @param bClearTransition - Clear any transitions.
	 */
	void SetSlide( float InSlide, bool bClearTransition = false)
	{
		SlideValue = InSlide;

		if (bClearTransition)
		{
			bPeak = false;
			bTransition = false;
			DesiredSlideValue = SlideValue;
		}
	}

	/**
	 * Set the desired move speed.
	 * @param DesiredMoveSpeed - the desired move speed.
	 */
	void SetMoveSpeed( float DesiredMoveSpeed )
	{
		MoveSpeed = DesiredMoveSpeed;
	}

	/**
	 * Set the desired left limit. The widget will go to this position when scrolling out and left.
	 * @param DesiredLeftLimit - the desired left limit.
	 */
	void SetSliderLeftLimit( float DesiredLeftLimit )
	{
		SlideValueLeftLimit = DesiredLeftLimit;
	}

	/**
	 * Set the desired right limit. The widget will go to this position when scrolling out and right.
	 * @param DesiredRightLimit - the desired right limit.
	 */
	void SetSliderRightLimit( float DesiredRightLimit )
	{
		SlideValueRightLimit = DesiredRightLimit;
	}

	void SetSliderPeakLeftLimit(float DesiredLimit)
	{
		PeakValueLeftLimit = DesiredLimit;
	}

	void SetSliderPeakRightLimit(float DesiredLimit)
	{
		PeakValueRightLimit = DesiredLimit;
	}

	/**
	 * Set the optimal slide position. The widget will go to this position when in view.
	 * @param DesiredOptimalSlideValue - the desired optimal position.
	 */
	void SetSliderOptimalPostion( float DesiredOptimalSlideValue )
	{
		OptimalSlideValue = DesiredOptimalSlideValue;
	}

	/**
	 * Set the fade rate. The widget fade in and out at this amount per frame.
	 * @param DesiredFadeRate - the desired fade value.
	 */
	void SetFadeRate( float DesiredFadeRate )
	{
		FadeRate = DesiredFadeRate;
	}

	bool IsInTransition()
	{
		return bTransition || bFade;
	}

	const float GetSlideValue() const
	{
		return SlideValue;
	}

	const float GetMoveSpeed() const
	{
		return MoveSpeed;
	}

protected:

	/**
	 * Get the slide position - used by the FX widget to get its position.
	 * @return The slide position.
	 */
	FVector2D GetSlide() const
	{
		return FVector2D( SlideValue, 0.f );
	}

protected:
	// Holds the Opacity Value. 0 == transparent
	float OpacityValue;

	// The slide position
	float SlideValue;

	// The desired opacity value
	float DesiredOpacityValue;

	// The max left position
	float SlideValueLeftLimit;

	// The max right position
	float SlideValueRightLimit;

	// The left peak position
	float PeakValueLeftLimit;

	// The right peak position
	float PeakValueRightLimit;

	// The slide value to display the widget on screen
	float OptimalSlideValue;

	// The desired slide value
	float DesiredSlideValue;

	// Holds if widget is in transition
	bool bTransition;

	// Hold if the widget is peaking
	bool bPeak;

	// Holds if the widget is fading in or out
	bool bFade;

	// Holds if the widget should fade in or out
	bool bFadeInAndOut;

	// The maximum move speed of the widget
	float MoveSpeed;

	// The Peak speed
	float PeakSpeed;

	// The Peak Distance
	float PeakDistance;

	// The amount by which a widget fades per frame
	float FadeRate;

	// Holds the FX widget that does the transitioning
	TSharedPtr< SFxWidget > FXWidget;

	// Holds the peak curve
	FCurveSequence SlideInCurve;

	// The current visibility of the display item
	EVisibility Visibility;
};

/**
 * Implements a widget Carousel.
 *
 * A widget Carousel displays widgets that can be scrolled in and out.
 */
template <typename ItemType>
class SWidgetCarousel
	: public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnCarouselPageChanged, int32 /*PageIndex*/);
	typedef typename TSlateDelegates< ItemType >::FOnGenerateWidget FOnGenerateWidget;

	SLATE_BEGIN_ARGS(SWidgetCarousel<ItemType>)
		: _WidgetItemsSource( static_cast<const TArray<ItemType>*>(NULL) )
		, _MoveSpeed( 1.f )
		, _SlideValueLeftLimit( -1.f )
		, _SlideValueRightLimit( 1.f )
		, _PeakValueLeftLimit(-0.05f)
		, _PeakValueRightLimit(0.05f)
		, _FadeRate( 1.0f )
	{
		this->_Clipping = EWidgetClipping::ClipToBounds;
	}

	/** Called when we change a widget */
	SLATE_EVENT( FOnGenerateWidget, OnGenerateWidget )
	/** Event triggered when the current page changes */
	SLATE_EVENT(FOnCarouselPageChanged, OnPageChanged)
	/** The data source */
	SLATE_ARGUMENT( const TArray<ItemType>*, WidgetItemsSource )
	/** The move speed */
	SLATE_ATTRIBUTE(float, MoveSpeed )
	/** The left limit */
	SLATE_ATTRIBUTE(float, SlideValueLeftLimit )
	/** The right limit */
	SLATE_ATTRIBUTE(float, SlideValueRightLimit )
	/** The left limit */
	SLATE_ATTRIBUTE(float, PeakValueLeftLimit)
	/** The right limit */
	SLATE_ATTRIBUTE(float, PeakValueRightLimit)
	/** The fade rate */
	SLATE_ATTRIBUTE(float, FadeRate )

	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 */
	void Construct( const FArguments& InArgs )
	{
		OnGenerateWidget = InArgs._OnGenerateWidget;
		OnPageChanged = InArgs._OnPageChanged;
		ItemsSource = InArgs._WidgetItemsSource;
		WidgetIndex = 0;
		MoveSpeed = InArgs._MoveSpeed.Get();
		SlideValueLeftLimit = InArgs._SlideValueLeftLimit.Get();
		SlideValueRightLimit = InArgs._SlideValueRightLimit.Get(); 
		PeakValueLeftLimit = InArgs._PeakValueLeftLimit.Get();
		PeakValueRightLimit = InArgs._PeakValueRightLimit.Get();

		FadeRate = InArgs._FadeRate.Get();

		LeftCarouselWidget = MakeShareable(new FCarouselDisplayItem());
		CenterCarouselWidget = MakeShareable(new FCarouselDisplayItem());
		RightCarouselWidget = MakeShareable(new FCarouselDisplayItem());

		// Set up the Carousel Widget
		CenterCarouselWidget->SetOpacity(1.0f);
		CenterCarouselWidget->SetSlide(0.f);
		CenterCarouselWidget->SetMoveSpeed(MoveSpeed);
		CenterCarouselWidget->SetFadeRate(FadeRate);

		LeftCarouselWidget->SetOpacity(1.0f);
		LeftCarouselWidget->SetSlide(SlideValueLeftLimit);
		LeftCarouselWidget->SetMoveSpeed(MoveSpeed);
		LeftCarouselWidget->SetFadeRate(FadeRate);

		RightCarouselWidget->SetOpacity(1.0f);
		RightCarouselWidget->SetSlide(SlideValueRightLimit);
		RightCarouselWidget->SetMoveSpeed(MoveSpeed);
		RightCarouselWidget->SetFadeRate(FadeRate);

		// Create the widget
		ChildSlot
		[
			SAssignNew(WidgetDisplayBox, SHorizontalBox)
		];

		GenerateCurrentWidgets();

		OnPageChanged.ExecuteIfBound(WidgetIndex);
	}

	void GenerateCurrentWidgets()
	{
		const TArray<ItemType>& ItemsSourceRef = (*ItemsSource);
		if (ItemsSourceRef.Num() > 0)
		{
			if (WidgetIndex >= ItemsSourceRef.Num())
			{
				WidgetIndex = WidgetIndex % ItemsSourceRef.Num();
			}
			
			TSharedRef<SWidget> LeftWidget = OnGenerateWidget.Execute(ItemsSourceRef[GetLeftWidgetIndex(WidgetIndex)]);
			TSharedRef<SWidget> RightWidget = OnGenerateWidget.Execute(ItemsSourceRef[GetRightWidgetIndex(WidgetIndex)]);
			TSharedRef<SWidget> CenterWidget = OnGenerateWidget.Execute(ItemsSourceRef[WidgetIndex]);

			// Each carousel widget must have unique content since Slate Global Invalidation requires that widgets only exist in a single slot
			const bool bIsLeftWidgetUnique = (LeftWidget != CenterWidget) && (RightWidget != LeftWidget);
			const bool bIsRightWidgetUnique = RightWidget != CenterWidget;
			LeftCarouselWidget->SetWidgetContent(bIsLeftWidgetUnique ? LeftWidget : SNullWidget::NullWidget);
			RightCarouselWidget->SetWidgetContent(bIsRightWidgetUnique ? RightWidget : SNullWidget::NullWidget);
			// CenterCarouselWidget is the one that displays the active child.  Needs to be populated last for the Widget Path to match the visible hierarchy.
			CenterCarouselWidget->SetWidgetContent(CenterWidget);
		}

		if (ItemsSourceRef.Num() == 1)
		{
			LeftCarouselWidget->SetVisibility(EVisibility::Collapsed);
			RightCarouselWidget->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			LeftCarouselWidget->SetVisibility(EVisibility::Visible);
			RightCarouselWidget->SetVisibility(EVisibility::Visible);
		}

		SetSliderLimits();
		GenerateWidgets();
	}

	/**
	 * Get the current widget index.
	 * @return The visible widget index
	 */
	int32 GetWidgetIndex()
	{
		return WidgetIndex;
	}

	/**
	 * Sets the active widget to display at the specified index.
	 *
	 * @param Index - The desired widget index.
	 */
	void SetActiveWidgetIndex(int32 Index)
	{
		if (!IsInTransition())
		{
			// Don't change if the widget index is the same
			if (Index != WidgetIndex)
			{
				// Choose the scroll direction
				EWidgetCarouselScrollDirection::Type ScrollDirection = Index > WidgetIndex ?
					EWidgetCarouselScrollDirection::Carousel_Right : EWidgetCarouselScrollDirection::Carousel_Left;

				SwapBuffer(ScrollDirection, Index);
			}
		}
	}

	/**
	 * Scroll right.
	 */
	void SetNextWidget()
	{
		if (!IsInTransition())
		{
			SwapBuffer(EWidgetCarouselScrollDirection::Carousel_Left);
		}
	}

	/**
	 * Scroll left.
	 */
	void SetPreviousWidget()
	{
		if (!IsInTransition())
		{
			SwapBuffer(EWidgetCarouselScrollDirection::Carousel_Right);
		}
	}

	void Peak(EWidgetCarouselScrollDirection::Type Direction)
	{
		if (!IsInTransition())
		{
			CenterCarouselWidget->PeakIn(Direction);
			RightCarouselWidget->PeakIn(Direction);
			LeftCarouselWidget->PeakIn(Direction);
		}
	}

	/**
	 * Set the item source, and scroll to the first item.
	 * @param The item source
	 */
	void SetItemSource( const TArray<ItemType>* InWidgetItemsSource )
	{
		ItemsSource = InWidgetItemsSource;
		if ( ItemsSource->Num() )
		{
			SetActiveWidgetIndex( 0 );
		}
	}

	float GetPrimarySlide()
	{
		return CenterCarouselWidget->GetSlideValue();
	}

	void SetMoveSpeed(float InMoveSpeed)
	{
		if (!IsInTransition())
		{
			CenterCarouselWidget->SetMoveSpeed(InMoveSpeed);
			RightCarouselWidget->SetMoveSpeed(InMoveSpeed);
			LeftCarouselWidget->SetMoveSpeed(InMoveSpeed);
		}
	}

	float GetMoveSpeed() const
	{
		return CenterCarouselWidget->GetMoveSpeed();
	}

	void ResetTransition()
	{
		constexpr bool bClearTransition = true;
		CenterCarouselWidget->SetSlide(0.f, bClearTransition);
		LeftCarouselWidget->SetSlide(SlideValueLeftLimit, bClearTransition);
		RightCarouselWidget->SetSlide(SlideValueRightLimit, bClearTransition);
	}

protected:

	/**
	 * Regenerate the widgets to display them.
	 */
	void GenerateWidgets( )
	{
		WidgetDisplayBox->ClearChildren();
		WidgetDisplayBox->AddSlot()
		[
			SNew( SOverlay )
			+SOverlay::Slot()
			[
				LeftCarouselWidget->GetSFXWidget().ToSharedRef()
			]
			+SOverlay::Slot()
			[
				RightCarouselWidget->GetSFXWidget().ToSharedRef()
			]
			+ SOverlay::Slot()
			[
				CenterCarouselWidget->GetSFXWidget().ToSharedRef()
			]
		];
	}

	/**
	 * Swap the buffer index.
	 */
	void SwapBuffer(EWidgetCarouselScrollDirection::Type ScrollDirection, int32 OverrideWidgetIndex = INDEX_NONE)
	{
		// Register for an active update every 2.5 seconds
		RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SWidgetCarousel::RegisteredTick));

		const TArray<ItemType>& ItemsSourceRef = (*ItemsSource);
		
		TSharedRef<SWidget> LeftWidget = LeftCarouselWidget->GetWidgetContent();
		TSharedRef<SWidget> RightWidget = RightCarouselWidget->GetWidgetContent();
		TSharedRef<SWidget> CenterWidget = CenterCarouselWidget->GetWidgetContent();

		const bool bIsScrollingLeft = ScrollDirection == EWidgetCarouselScrollDirection::Carousel_Left;

		if (ItemsSourceRef.Num() > 0)
		{
			if (bIsScrollingLeft)
			{
				WidgetIndex = GetLeftWidgetIndex(WidgetIndex);
				TSharedPtr<FCarouselDisplayItem> TempWidget = RightCarouselWidget;
				RightCarouselWidget = CenterCarouselWidget;
				RightWidget = (CenterWidget == SNullWidget::NullWidget) ? OnGenerateWidget.Execute(ItemsSourceRef[GetRightWidgetIndex(WidgetIndex)]) : CenterWidget;
				CenterCarouselWidget = LeftCarouselWidget;
				CenterWidget = (LeftWidget == SNullWidget::NullWidget) ? OnGenerateWidget.Execute(ItemsSourceRef[WidgetIndex]) : LeftWidget;
				LeftCarouselWidget = TempWidget;
				LeftWidget = OnGenerateWidget.Execute(ItemsSourceRef[GetLeftWidgetIndex(WidgetIndex)]);
				LeftCarouselWidget->SetSlide(CenterCarouselWidget->GetSlideValue() + SlideValueLeftLimit, true);
			}
			else
			{
				WidgetIndex = GetRightWidgetIndex(WidgetIndex);
				TSharedPtr<FCarouselDisplayItem> TempWidget = LeftCarouselWidget;
				LeftCarouselWidget = CenterCarouselWidget;
				LeftWidget = (CenterWidget == SNullWidget::NullWidget) ? OnGenerateWidget.Execute(ItemsSourceRef[GetLeftWidgetIndex(WidgetIndex)]) : CenterWidget;
				CenterCarouselWidget = RightCarouselWidget;
				CenterWidget = (RightWidget == SNullWidget::NullWidget) ? OnGenerateWidget.Execute(ItemsSourceRef[WidgetIndex]) : RightWidget;
				RightCarouselWidget = TempWidget;
				RightWidget = OnGenerateWidget.Execute(ItemsSourceRef[GetRightWidgetIndex(WidgetIndex)]);
				RightCarouselWidget->SetSlide(CenterCarouselWidget->GetSlideValue() + SlideValueRightLimit, true);
			}
		}

		if (ItemsSourceRef.IsValidIndex(OverrideWidgetIndex))
		{
			// Need to find a way to clear left and right images
			WidgetIndex = OverrideWidgetIndex;
			CenterWidget = OnGenerateWidget.Execute(ItemsSourceRef[OverrideWidgetIndex]);
		}
		
		// Ensure the 3 CarouselWidgets here have unique widgets, if that isn't the case, prioritize the 2 widgets that will display during the transition
		const bool bIsLeftWidgetUnique = (LeftWidget != CenterWidget) && (!bIsScrollingLeft || RightWidget != LeftWidget);
		const bool bIsRightWidgetUnique = (RightWidget != CenterWidget) && (bIsScrollingLeft || RightWidget != LeftWidget);
		LeftCarouselWidget->SetWidgetContent(bIsLeftWidgetUnique ? LeftWidget : SNullWidget::NullWidget);
		RightCarouselWidget->SetWidgetContent(bIsRightWidgetUnique ? RightWidget : SNullWidget::NullWidget);
		CenterCarouselWidget->SetWidgetContent(CenterWidget);

		SetSliderLimits();
		GenerateWidgets();
		CenterCarouselWidget->ScrollIn(ScrollDirection);
		RightCarouselWidget->ScrollIn(ScrollDirection);
		LeftCarouselWidget->ScrollIn(ScrollDirection);

		OnPageChanged.ExecuteIfBound(WidgetIndex);
	}

	void SetSliderLimits()
	{
		LeftCarouselWidget->SetSliderLeftLimit(SlideValueLeftLimit);
		LeftCarouselWidget->SetSliderRightLimit(SlideValueRightLimit);
		LeftCarouselWidget->SetSliderPeakLeftLimit(SlideValueLeftLimit + PeakValueLeftLimit);
		LeftCarouselWidget->SetSliderPeakRightLimit(SlideValueLeftLimit + PeakValueRightLimit);
		LeftCarouselWidget->SetSliderOptimalPostion(SlideValueLeftLimit);

		RightCarouselWidget->SetSliderLeftLimit(SlideValueLeftLimit);
		RightCarouselWidget->SetSliderRightLimit(SlideValueRightLimit);
		RightCarouselWidget->SetSliderPeakLeftLimit(SlideValueRightLimit + PeakValueLeftLimit);
		RightCarouselWidget->SetSliderPeakRightLimit(SlideValueRightLimit + PeakValueRightLimit);
		RightCarouselWidget->SetSliderOptimalPostion(SlideValueRightLimit);

		CenterCarouselWidget->SetSliderLeftLimit(SlideValueLeftLimit);
		CenterCarouselWidget->SetSliderRightLimit(SlideValueRightLimit);
		CenterCarouselWidget->SetSliderPeakLeftLimit(PeakValueLeftLimit);
		CenterCarouselWidget->SetSliderPeakRightLimit(PeakValueRightLimit);
		CenterCarouselWidget->SetSliderOptimalPostion(0);
	}

	int32 GetLeftWidgetIndex(int32 Index)
	{
		int32 LeftWidgetIndex = Index - 1;
		if (LeftWidgetIndex < 0)
		{
			LeftWidgetIndex = FMath::Max(0, ItemsSource->Num() - 1);
		}
		return LeftWidgetIndex;
	}

	int32 GetRightWidgetIndex(int32 Index)
	{
		int32 RightWidgetIndex = Index + 1;
		if (RightWidgetIndex >= ItemsSource->Num())
		{
			RightWidgetIndex = 0;
		}
		return RightWidgetIndex;
	}

	bool IsInTransition()
	{
		return CenterCarouselWidget->IsInTransition() || LeftCarouselWidget->IsInTransition() || RightCarouselWidget->IsInTransition();
	}

	EActiveTimerReturnType RegisteredTick(double InCurrentTime, float InDeltaTime)
	{
		// Tick the widgets to animate them
		LeftCarouselWidget->Tick(InDeltaTime);
		CenterCarouselWidget->Tick(InDeltaTime);
		RightCarouselWidget->Tick(InDeltaTime);
	
		return IsInTransition() ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
	}

protected:

	// Holds the CarouselWidgets
	TSharedPtr< FCarouselDisplayItem > LeftCarouselWidget;
	TSharedPtr< FCarouselDisplayItem > CenterCarouselWidget;
	TSharedPtr< FCarouselDisplayItem > RightCarouselWidget;

	// Pointer to the array of data items that we are observing
	const TArray<ItemType>* ItemsSource;

	// Hold the delegate to be invoked when a widget changes.
	FOnGenerateWidget OnGenerateWidget;

	// Hold the delegate to be invoked when the current page changes.
	FOnCarouselPageChanged OnPageChanged;

	// Holds the widget display box.
	TSharedPtr< SHorizontalBox > WidgetDisplayBox;

	// Holds the visible widget index.
	int32 WidgetIndex;

	// Holds the move speed
	float MoveSpeed;

	// Holds The left limit
	float SlideValueLeftLimit;

	// The right limit
	float SlideValueRightLimit;

	// Holds The peak left limit
	float PeakValueLeftLimit;

	// The peak right limit
	float PeakValueRightLimit;

	// The fade rate
	float FadeRate;
};
