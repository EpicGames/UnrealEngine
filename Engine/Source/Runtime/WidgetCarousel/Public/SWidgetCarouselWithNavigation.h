// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "WidgetCarouselStyle.h"
#include "Widgets/SOverlay.h"
#include "SWidgetCarousel.h"
#include "SCarouselNavigationButton.h"
#include "SCarouselNavigationBar.h"

#include "Widgets/Layout/SScaleBox.h"

/** A widget carousel which includes embedded left and right navigation buttons and a navigation bar below. */
template <typename ItemType>
class SWidgetCarouselWithNavigation
	: public SCompoundWidget
{
public:
	typedef typename TSlateDelegates< ItemType >::FOnGenerateWidget FOnGenerateWidget;

	SLATE_BEGIN_ARGS(SWidgetCarouselWithNavigation<ItemType>)
		: _NavigationBarStyle()
		, _NavigationButtonStyle()
		, _WidgetItemsSource(static_cast<const TArray<ItemType>*>(NULL))
	{ }

		SLATE_STYLE_ARGUMENT(FWidgetCarouselNavigationBarStyle, NavigationBarStyle)

		SLATE_STYLE_ARGUMENT(FWidgetCarouselNavigationButtonStyle, NavigationButtonStyle)

		/** Called when we change a widget */
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

		/** The data source */
		SLATE_ARGUMENT(const TArray<ItemType>*, WidgetItemsSource)

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		OnGenerateWidget = InArgs._OnGenerateWidget;
		WidgetItemsSource = InArgs._WidgetItemsSource;

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MaxDesiredHeight(442)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.BorderImage(FWidgetCarouselModuleStyle::Get().GetBrush("WidgetBackground"))
						.Padding(0)
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							.StretchDirection(EStretchDirection::DownOnly)
							[
								SAssignNew(Carousel, SWidgetCarousel<ItemType>)
								.WidgetItemsSource(WidgetItemsSource)
								.FadeRate(0)
								.SlideValueLeftLimit(-1)
								.SlideValueRightLimit(1)
								.MoveSpeed(5.f)
								.OnGenerateWidget(this, &SWidgetCarouselWithNavigation::GenerateWidget)
							]
						]
					]

					+ SOverlay::Slot()
					.Padding(0)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						[
							SNew(SCarouselNavigationButton)
							.Style(InArgs._NavigationButtonStyle)
							.OnClicked(this, &SWidgetCarouselWithNavigation::HandleNextButtonClicked)
							.Visibility(this, &SWidgetCarouselWithNavigation::GetScreenshotNavigationVisibility)
							.OnBeginPeek(this, &SWidgetCarouselWithNavigation::HandlePeak, EWidgetCarouselScrollDirection::Carousel_Right)
							.OnEndPeek(this, &SWidgetCarouselWithNavigation::HandlePeak, EWidgetCarouselScrollDirection::Carousel_Center)
							.Direction(SCarouselNavigationButton::ENavigationButtonDirection::Left)
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							SNew(SCarouselNavigationButton)
							.Style(InArgs._NavigationButtonStyle)
							.OnClicked(this, &SWidgetCarouselWithNavigation::HandlePreviousButtonClicked)
							.Visibility(this, &SWidgetCarouselWithNavigation::GetScreenshotNavigationVisibility)
							.OnBeginPeek(this, &SWidgetCarouselWithNavigation::HandlePeak, EWidgetCarouselScrollDirection::Carousel_Left)
							.OnEndPeek(this, &SWidgetCarouselWithNavigation::HandlePeak, EWidgetCarouselScrollDirection::Carousel_Center)
							.Direction(SCarouselNavigationButton::ENavigationButtonDirection::Right)
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(10.0f, 10.f)
			[
				SNew(SCarouselNavigationBar)
				.Style(InArgs._NavigationBarStyle)
				.Visibility(this, &SWidgetCarouselWithNavigation::GetScreenshotNavigationVisibility)
				.ItemCount(WidgetItemsSource->Num())
				.CurrentItemIndex(this, &SWidgetCarouselWithNavigation::GetCurrentItemIndex)
				.CurrentSlideAmount(this, &SWidgetCarouselWithNavigation::GetCurrentSlideAmount)
				.OnSelectedIndexChanged(this, &SWidgetCarouselWithNavigation::CarouselScrollBarIndexChanged)
			]
		];
	}

	~SWidgetCarouselWithNavigation() = default;

private:
	TSharedRef<SWidget> GenerateWidget(ItemType Item)
	{
		return OnGenerateWidget.Execute(Item);
	}

	FReply HandleNextButtonClicked()
	{
		Carousel->SetNextWidget();
		return FReply::Handled();
	}

	FReply HandlePreviousButtonClicked()
	{
		Carousel->SetPreviousWidget();
		return FReply::Handled();
	}

	void HandlePeak(EWidgetCarouselScrollDirection::Type Direction)
	{
		Carousel->Peak(Direction);
	}

	EVisibility GetScreenshotVisibility() const
	{
		return WidgetItemsSource->Num() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetScreenshotNavigationVisibility() const
	{
		return WidgetItemsSource->Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
	}

	int32 GetCurrentItemIndex() const
	{
		return Carousel->GetWidgetIndex();
	}

	float GetCurrentSlideAmount() const
	{
		return Carousel->GetPrimarySlide();
	}

	void CarouselScrollBarIndexChanged(int32 NewIndex)
	{
		Carousel->SetActiveWidgetIndex(NewIndex);
	}

private:
	TSharedPtr<SWidgetCarousel<ItemType>> Carousel;
	FOnGenerateWidget OnGenerateWidget;
	const TArray<ItemType>* WidgetItemsSource;
};
