// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Implements the WidgetCarousel module.
 */
class FWidgetCarouselModule
	: public IModuleInterface
{
public:

	WIDGETCAROUSEL_API virtual void StartupModule( ) override;

	WIDGETCAROUSEL_API virtual void ShutdownModule( ) override;
};
