// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "WidgetCarouselStyle.generated.h"

USTRUCT()
struct FWidgetCarouselNavigationButtonStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	WIDGETCAROUSEL_API FWidgetCarouselNavigationButtonStyle();

	virtual ~FWidgetCarouselNavigationButtonStyle() = default;

	WIDGETCAROUSEL_API virtual void GetResources(TArray<const FSlateBrush*> & OutBrushes) const override;

	static WIDGETCAROUSEL_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static WIDGETCAROUSEL_API const FWidgetCarouselNavigationButtonStyle& GetDefault();

	UPROPERTY()
	FButtonStyle InnerButtonStyle;
	FWidgetCarouselNavigationButtonStyle& SetInnerButtonStyle(const FButtonStyle& InInnerButtonStyle) { InnerButtonStyle = InInnerButtonStyle; return *this; }

	UPROPERTY()
	FSlateBrush NavigationButtonLeftImage;
	FWidgetCarouselNavigationButtonStyle& SetNavigationButtonLeftImage(const FSlateBrush& InNavigationButtonLeftImage) { NavigationButtonLeftImage = InNavigationButtonLeftImage; return *this; }

	UPROPERTY()
	FSlateBrush NavigationButtonRightImage;
	FWidgetCarouselNavigationButtonStyle& SetNavigationButtonRightImage(const FSlateBrush& InNavigationButtonRightImage) { NavigationButtonRightImage = InNavigationButtonRightImage; return *this; }
};

USTRUCT(BlueprintType)
struct FWidgetCarouselNavigationBarStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	WIDGETCAROUSEL_API FWidgetCarouselNavigationBarStyle();

	virtual ~FWidgetCarouselNavigationBarStyle() = default;

	WIDGETCAROUSEL_API virtual void GetResources(TArray<const FSlateBrush*> & OutBrushes) const override;

	static WIDGETCAROUSEL_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static WIDGETCAROUSEL_API const FWidgetCarouselNavigationBarStyle& GetDefault();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HighlightBrush;
	FWidgetCarouselNavigationBarStyle& SetHighlightBrush(const FSlateBrush& InHighlightBrush) { HighlightBrush = InHighlightBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle LeftButtonStyle;
	FWidgetCarouselNavigationBarStyle& SetLeftButtonStyle(const FButtonStyle& InLeftButtonStyle) { LeftButtonStyle = InLeftButtonStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle CenterButtonStyle;
	FWidgetCarouselNavigationBarStyle& SetCenterButtonStyle(const FButtonStyle& InCenterButtonStyle) { CenterButtonStyle = InCenterButtonStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle RightButtonStyle;
	FWidgetCarouselNavigationBarStyle& SetRightButtonStyle(const FButtonStyle& InRightButtonStyle) { RightButtonStyle = InRightButtonStyle; return *this; }
};

/** Manages the style which provides resources for the add content dialog. */
class FWidgetCarouselModuleStyle
{
public:

	static WIDGETCAROUSEL_API void Initialize();

	static WIDGETCAROUSEL_API void Shutdown();

	/** reloads textures used by slate renderer */
	static WIDGETCAROUSEL_API void ReloadTextures();

	/** @return The Slate style set for the widget carousel */
	static WIDGETCAROUSEL_API const ISlateStyle& Get();

	static WIDGETCAROUSEL_API FName GetStyleSetName();

private:

	static TSharedRef< class FSlateStyleSet > Create();

private:

	static TSharedPtr< class FSlateStyleSet > WidgetCarouselStyleInstance;
};
